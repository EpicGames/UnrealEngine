// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "ISourceControlWindowExtenderModule.h"
#include "ISourceControlWindowsModule.h"
#include "SourceControlMenuContext.h"
#include "SourceControlHelpers.h"
#include "ToolMenus.h"
#include "AssetRegistry/AssetData.h"
#include "GameFramework/Actor.h"
#include "Editor.h"
#include "Selection.h"
#include "ScopedTransaction.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"

#define LOCTEXT_NAMESPACE "SourceControlWindowExtender"

/**
 * SourceControlWindowExtender module
 */
class FSourceControlWindowExtenderModule : public ISourceControlWindowExtenderModule
{
public:
	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override;

private:
	void ExtendMenu();

	bool CanPinActors() const { return CurrentWorldUnloadedActors.Num() > 0; }
	void PinActors();
	void PinActors(const TArray<FAssetData>& ActorsToPin);
	bool CanSelectActors() const { return CurrentWorldLoadedActors.Num() > 0; }
	void SelectActors();
	void SelectActors(const TArray<FAssetData>& ActorsToSelect);
	void BrowseToAssets();
	void BrowseToAssets(const TArray<FAssetData>& Assets);
	void FocusActors(const TArray<FAssetData>& ActorToFocus, bool bSelect);
	bool CanFocusActors() const { return CurrentWorldLoadedActors.Num() > 0 || CurrentWorldUnloadedActors.Num() > 0; }
	void FocusActors();

	void OnChangelistFileDoubleClicked(const FString& Filename);

	void GetAssetsFromFilenames(const TArray<FString>& Filenames, TArray<FAssetData>& OutNonActorAssets, TArray<FAssetData>& OutCurrentWorldLoadedActors, TArray<FAssetData>& OutCurrentWorldUnloadedActors);

	TArray<FAssetData> SelectedAssets;
	TArray<FAssetData> CurrentWorldLoadedActors;
	TArray<FAssetData> CurrentWorldUnloadedActors;
	FDelegateHandle EventHandle;
};

IMPLEMENT_MODULE(FSourceControlWindowExtenderModule, SourceControlWindowExtender);

void FSourceControlWindowExtenderModule::StartupModule()
{
	ExtendMenu();

	EventHandle = ISourceControlWindowsModule::Get().OnChangelistFileDoubleClicked().AddRaw(this, &FSourceControlWindowExtenderModule::OnChangelistFileDoubleClicked);
}

void FSourceControlWindowExtenderModule::ShutdownModule()
{
	if (ISourceControlWindowsModule* Module = ISourceControlWindowsModule::TryGet())
	{
		Module->OnChangelistFileDoubleClicked().Remove(EventHandle);
	}
}

void FSourceControlWindowExtenderModule::GetAssetsFromFilenames(const TArray<FString>& Filenames, TArray<FAssetData>& OutNonActorAssets, TArray<FAssetData>& OutCurrentWorldLoadedActors, TArray<FAssetData>& OutCurrentWorldUnloadedActors)
{
	UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();

	for (const FString& Filename : Filenames)
	{
		TArray<FAssetData> OutAssets;
		if (SourceControlHelpers::GetAssetData(Filename, OutAssets) && OutAssets.Num() == 1)
		{
			const FAssetData& AssetData = OutAssets[0];
			if (TSubclassOf<AActor> ActorClass = AssetData.GetClass())
			{
				if (CurrentWorld && AssetData.GetObjectPathString().StartsWith(CurrentWorld->GetPathName()))
				{
					if (AssetData.IsAssetLoaded())
					{
						OutCurrentWorldLoadedActors.Add(AssetData);
					}
					else
					{
						OutCurrentWorldUnloadedActors.Add(AssetData);
					}
				}
				else
				{
					TArray<FAssetData> OutWorldAsset;
					FString AssetPathName = AssetData.ToSoftObjectPath().GetLongPackageName();
					if (SourceControlHelpers::GetAssetDataFromPackage(AssetPathName, OutWorldAsset) && OutWorldAsset.Num() == 1)
					{
						OutNonActorAssets.Add(OutWorldAsset[0]);
					}
				}
			}
			else
			{
				OutNonActorAssets.Add(AssetData);
			}
		}
	}
}

void FSourceControlWindowExtenderModule::ExtendMenu()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("SourceControl.ChangelistContextMenu");
	FToolMenuSection& Section = Menu->AddDynamicSection("SourceControlWindowExtenderSection", FNewToolMenuDelegate::CreateLambda([this](UToolMenu* InMenu)
	{
		if (USourceControlMenuContext* MenuContext = InMenu->FindContext<USourceControlMenuContext>())
        {
            SelectedAssets.Empty();
            CurrentWorldLoadedActors.Empty();
            CurrentWorldUnloadedActors.Empty();

			GetAssetsFromFilenames(MenuContext->SelectedFiles, SelectedAssets, CurrentWorldLoadedActors, CurrentWorldUnloadedActors);

			auto FindOrAddExtendSection = [InMenu]() -> FToolMenuSection*
			{
				FToolMenuSection* ExtendSection = InMenu->FindSection("SourceControlWindowExtender");
				if (!ExtendSection)
				{
					ExtendSection = &InMenu->AddSection("SourceControlWindowExtender", TAttribute<FText>(), FToolMenuInsert("Source Control", EToolMenuInsertType::After));
					ExtendSection->AddSeparator(NAME_None);
				}
				return ExtendSection;
			};

            if (CurrentWorldUnloadedActors.Num() > 0 || CurrentWorldLoadedActors.Num() > 0)
            {
				FindOrAddExtendSection()->AddSubMenu(TEXT("Actors"),
                    LOCTEXT("ActorSubMenu", "Actors"),
                    FText(),
                    FNewToolMenuChoice(FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
                    {
                        MenuBuilder.AddMenuEntry(LOCTEXT("PinActors", "Pin"), LOCTEXT("PinActors_Tooltip", "Load actors"),
							FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(this, &FSourceControlWindowExtenderModule::PinActors), FCanExecuteAction::CreateRaw(this, &FSourceControlWindowExtenderModule::CanPinActors)));
                        MenuBuilder.AddMenuEntry(LOCTEXT("SelectActors", "Select"), LOCTEXT("SelectActors_Tooltip", "Select actors"), 
							FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(this, &FSourceControlWindowExtenderModule::SelectActors), FCanExecuteAction::CreateRaw(this, &FSourceControlWindowExtenderModule::CanSelectActors)));
						MenuBuilder.AddMenuEntry(LOCTEXT("FocusActors", "Focus"), LOCTEXT("FocusActors_Tooltip", "Focus actors"),
							FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(this, &FSourceControlWindowExtenderModule::FocusActors), FCanExecuteAction::CreateRaw(this, &FSourceControlWindowExtenderModule::CanFocusActors)));
                    })));
            }

            if (SelectedAssets.Num() > 0)
            {
				FindOrAddExtendSection()->AddSubMenu(TEXT("Assets"),
                    LOCTEXT("AssetSubMenu", "Assets"),
                    FText(),
                    FNewToolMenuChoice(FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
                    {
                        MenuBuilder.BeginSection(NAME_None);
                        MenuBuilder.AddMenuEntry(LOCTEXT("BrowseToAssets", "Browse to Asset"), LOCTEXT("BrowseToAssets_Tooltip", "Browse to Asset in Content Browser"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser.Small"), FUIAction(FExecuteAction::CreateRaw(this, &FSourceControlWindowExtenderModule::BrowseToAssets)));
                        MenuBuilder.EndSection();
                    })));
            }
        }
    }));
}

void FSourceControlWindowExtenderModule::PinActors()
{
	PinActors(CurrentWorldUnloadedActors);
}

void FSourceControlWindowExtenderModule::PinActors(const TArray<FAssetData>& ActorsToPin)
{
	UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();
	check(CurrentWorld);
	if (UWorldPartition* WorldPartition = CurrentWorld->GetWorldPartition())
	{
		TArray<FGuid> ActorGuids;
		for (const FAssetData& ActorToPin : ActorsToPin)
		{
			if(TUniquePtr<FWorldPartitionActorDesc> ActorDesc = FWorldPartitionActorDescUtils::GetActorDescriptorFromAssetData(ActorToPin))
			{
				ActorGuids.Add(ActorDesc->GetGuid());
			}
		}

		if (!ActorGuids.IsEmpty())
		{
			WorldPartition->PinActors(ActorGuids);
			SelectActors(CurrentWorldUnloadedActors);
		}
	}
}

void FSourceControlWindowExtenderModule::SelectActors()
{
	SelectActors(CurrentWorldLoadedActors);
}

void FSourceControlWindowExtenderModule::SelectActors(const TArray<FAssetData>& ActorsToSelect)
{
	const FScopedTransaction Transaction(LOCTEXT("SelectActorsFromChangelist", "Select Actor(s)"));
	UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();
	check(CurrentWorld);

	GEditor->GetSelectedActors()->BeginBatchSelectOperation();
	bool bNotify = false;
	const bool bDeselectBSPSurfs = true;
	GEditor->SelectNone(bNotify, bDeselectBSPSurfs);
	
	for (const FAssetData& ActorToSelect : ActorsToSelect)
	{
		if (AActor* Actor = Cast<AActor>(ActorToSelect.FastGetAsset()))
		{
			const bool bSelected = true;
			GEditor->SelectActor(Actor, bSelected, bNotify);
		}
	}

	bNotify = true;
	GEditor->GetSelectedActors()->EndBatchSelectOperation(bNotify);
}

void FSourceControlWindowExtenderModule::BrowseToAssets()
{
	BrowseToAssets(SelectedAssets);
}

void FSourceControlWindowExtenderModule::BrowseToAssets(const TArray<FAssetData>& Assets)
{
	GEditor->SyncBrowserToObjects(const_cast<TArray<FAssetData>&>(Assets));
}

void FSourceControlWindowExtenderModule::FocusActors()
{
	TArray<FAssetData> ActorsToFocus;
	ActorsToFocus.Append(CurrentWorldLoadedActors);
	ActorsToFocus.Append(CurrentWorldUnloadedActors);

	const bool bSelectActors = false;
	FocusActors(ActorsToFocus, bSelectActors);
}

void FSourceControlWindowExtenderModule::FocusActors(const TArray<FAssetData>& ActorsToFocus, bool bSelectActors)
{
	FBox FocusBounds(EForceInit::ForceInit);
	UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();
	check(CurrentWorld);
	for (const FAssetData& ActorToFocus : ActorsToFocus)
	{
		if (TUniquePtr<FWorldPartitionActorDesc> ActorDesc = FWorldPartitionActorDescUtils::GetActorDescriptorFromAssetData(ActorToFocus))
		{
			const FBox EditorBounds = ActorDesc->GetEditorBounds();
			if (EditorBounds.IsValid)
			{
				FocusBounds += EditorBounds;
			}
		}
	}

	if (FocusBounds.IsValid)
	{
		const bool bActiveViewportOnly = true;
		const float TimeInSeconds = 0.5f;
		GEditor->MoveViewportCamerasToBox(FocusBounds, bActiveViewportOnly, TimeInSeconds);
	}

	if (bSelectActors)
	{
		SelectActors(ActorsToFocus);
	}
}

void FSourceControlWindowExtenderModule::OnChangelistFileDoubleClicked(const FString& Filename)
{
	TArray<FAssetData> OutNonActorAssets;
	TArray<FAssetData> OutCurrentWorldLoadedActors;
	TArray<FAssetData> OutCurrentWorldUnloadedActors;

	GetAssetsFromFilenames({ Filename }, OutNonActorAssets, OutCurrentWorldLoadedActors, OutCurrentWorldUnloadedActors);

	if (OutNonActorAssets.Num() > 0)
	{
		BrowseToAssets(OutNonActorAssets);
	}
	else if (OutCurrentWorldUnloadedActors.Num() > 0)
	{
		const bool bSelectActors = false;
		FocusActors(OutCurrentWorldUnloadedActors, bSelectActors);
	}
	else if (OutCurrentWorldLoadedActors.Num() > 0)
	{
		const bool bSelectActors = true;
		FocusActors(OutCurrentWorldLoadedActors, bSelectActors);
	}
}

#undef LOCTEXT_NAMESPACE