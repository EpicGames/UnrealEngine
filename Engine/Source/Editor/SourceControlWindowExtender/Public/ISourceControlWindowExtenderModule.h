// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

/**
 * SourceControlWindowExtender module interface
 */
class ISourceControlWindowExtenderModule : public IModuleInterface
{
public:
	/**
	 * Get reference to the SourceControlWindowExtender module instance
	 */
	static inline ISourceControlWindowExtenderModule& Get()
	{
		return FModuleManager::LoadModuleChecked<ISourceControlWindowExtenderModule>("SourceControlWindowExtender");
	}
};
