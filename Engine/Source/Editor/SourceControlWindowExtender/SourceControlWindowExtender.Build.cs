// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class SourceControlWindowExtender : ModuleRules
{
	public SourceControlWindowExtender(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core", 
				"CoreUObject", 
				"Engine",
            	"ToolMenus",
				"SlateCore",
				"Slate",
				"SourceControl",
				"SourceControlWindows",
				"UnrealEd"
			}
		);
	}
}
